// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PLUGIN_NAMEGenerated : ModuleRules
{
	public PLUGIN_NAMEGenerated(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "CoreUObject",
                "DeveloperSettings",
                "Json",
                "JsonUtilities",
                "WebAPI",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"HTTP",
				"Slate",
				"SlateCore",
			});
	}
}
